// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlComponentsContext.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "RemoteControlPreset.h"
#include "RemoteControlTrackerComponent.h"

FRemoteControlComponentsContext::FRemoteControlComponentsContext(UWorld* InWorld, URemoteControlPreset* InPreset)
	: WorldWeak(InWorld)
	, PresetWeak(InPreset)
{
}

bool FRemoteControlComponentsContext::RegisterActor(AActor* InActor)
{
	if (CanRegisterActor(InActor))
	{
		TrackedActors.Add(InActor);
		return true;
	}

	return false;
}

bool FRemoteControlComponentsContext::UnregisterActor(AActor* InActor)
{
	return TrackedActors.Remove(InActor) > 0;
}

bool FRemoteControlComponentsContext::IsActorRegistered(const AActor* InActor) const
{
	return InActor && TrackedActors.Contains(InActor);
}

bool FRemoteControlComponentsContext::CanRegisterActor(const AActor* InActor) const
{
	if (!InActor)
	{
		return false;
	}

	if (IsActorRegistered(InActor))
	{
		return false;
	}
	
	const UWorld* ActorWorld = InActor->GetWorld();
	const UWorld* World = GetWorld();
	if (!ActorWorld || ActorWorld != World)
	{
		return false;
	}

	return !!InActor->FindComponentByClass<URemoteControlTrackerComponent>();
}

bool FRemoteControlComponentsContext::IsValid() const
{
	return WorldWeak.IsValid() && PresetWeak.IsValid();
}

bool FRemoteControlComponentsContext::HasTrackedActors() const
{
	return !TrackedActors.IsEmpty();
}

UWorld* FRemoteControlComponentsContext::GetWorld() const
{
	return WorldWeak.Get();
}

URemoteControlPreset* FRemoteControlComponentsContext::GetPreset() const
{
	return PresetWeak.Get();
}
