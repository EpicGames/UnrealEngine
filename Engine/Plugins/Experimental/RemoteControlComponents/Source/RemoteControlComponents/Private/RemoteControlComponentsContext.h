// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class URemoteControlPreset;
class UWorld;

/**
 * Remote Control Components Tracking Context: represents a World with a set of Actors which have properties exposed to a specific Preset.
 * It currently works under the assumption that there's a single Preset per World, and that this info is runtime only. Both aspects might change in the future.
 *
 * Context is represented by:
 *	- a World
 *	- a Remote Control Preset
 *	- Tracked Actors with properties exposed to the Preset
 */
struct FRemoteControlComponentsContext
{
public:
	/** Create a Remote Control Context by passing a World and a Preset */
	FRemoteControlComponentsContext(UWorld* InWorld, URemoteControlPreset* InPreset);

	/** Adds an Actor to the list of tracked actors */
	bool RegisterActor(AActor* InActor);

	/** Removes an Actor from the list of tracked actors */
	bool UnregisterActor(AActor* InActor);

	/** Checks if the specified Actor is tracked or not */
	bool IsActorRegistered(const AActor* InActor) const;

	/** An Actor can be registered if it has a Remote Control Tracker Component and its World and the World of this context coincide */
	bool CanRegisterActor(const AActor* InActor) const;

	/** Context is valid if both World and Preset are valid */
	bool IsValid() const;

	/** True if the context currently has registered tracked actors */
	bool HasTrackedActors() const;

	/** Get the array of currently tracked actors */
	TArray<TWeakObjectPtr<AActor>> GetTrackedActors() const { return TrackedActors; }

	/** Get the World referenced by this context */
	UWorld* GetWorld() const;

	/** Get the Remote Control Preset referenced by this context */
	URemoteControlPreset* GetPreset() const;

private:
	TWeakObjectPtr<UWorld> WorldWeak;
	
	TWeakObjectPtr<URemoteControlPreset> PresetWeak;
	
	TArray<TWeakObjectPtr<AActor>> TrackedActors;

	bool bIsValid = false;
};
