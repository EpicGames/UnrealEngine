// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlComponentsUtils.h"
#include "RemoteControlBinding.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "RemoteControlTrackerComponent.h"
#include "Subsystems/RemoteControlComponentsSubsystem.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "RemoteControlComponentsUtils"

FGuid FRemoteControlComponentsUtils::GetPresetPropertyGuid(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FProperty* InProperty)
{
	const TArray<TWeakPtr<FRemoteControlProperty>>& RemoteControlExposedProperties = InPreset->GetExposedEntities<FRemoteControlProperty>();
	for (const TWeakPtr<FRemoteControlProperty>& RCPropertyWeak : RemoteControlExposedProperties)
	{
		if (const TSharedPtr<FRemoteControlProperty>& RCProperty = RCPropertyWeak.Pin())
		{
			const bool bIsExposed = InProperty->GetFName() == RCProperty->FieldName
				&& RCProperty->GetBindings().ContainsByPredicate([&InOwnerObject](const TWeakObjectPtr<URemoteControlBinding>& InBinding)
				{
					return InBinding.IsValid() && InBinding->Resolve() == InOwnerObject;
				});

			if (bIsExposed)
			{
				return RCProperty->GetId();
			}
		}
	}

	return FGuid();
}

URemoteControlTrackerComponent* FRemoteControlComponentsUtils::GetTrackerComponent(AActor* InActor, bool bInAddTrackerIfMissing /* = false*/)
{
	if (!InActor)
	{
		return nullptr;
	}
	
	if (!InActor->FindComponentByClass<URemoteControlTrackerComponent>() && bInAddTrackerIfMissing)
	{
		constexpr bool bShouldTransact = false;
		FRemoteControlComponentsUtils::AddTrackerComponent({InActor}, bShouldTransact);
	}

	URemoteControlTrackerComponent* TrackerComponent = InActor->GetComponentByClass<URemoteControlTrackerComponent>();

	return TrackerComponent;
}

URemoteControlPreset* FRemoteControlComponentsUtils::GetCurrentPreset(const UObject* InObject)
{
	if (const URemoteControlComponentsSubsystem* RemoteControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
	{
		return RemoteControlComponentsSubsystem->GetRegisteredPreset(InObject);
	}

	return nullptr;
}

URemoteControlPreset* FRemoteControlComponentsUtils::GetCurrentPreset(const UWorld* InWorld)
{
	if (const URemoteControlComponentsSubsystem* RemoteControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
	{
		return RemoteControlComponentsSubsystem->GetRegisteredPreset(InWorld);
	}

	return nullptr;
}

void FRemoteControlComponentsUtils::AddTrackerComponent(const TSet<TWeakObjectPtr<AActor>>& InActors, bool bInShouldTransact)
{
#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> AddTrackerTransaction;
	if (bInShouldTransact)
	{
		AddTrackerTransaction = MakeShared<FScopedTransaction>(LOCTEXT("AddTrackerComponents", "Add Tracker Component(s)"));
	}
#endif
	
	for (TWeakObjectPtr<AActor> ActorWeak : InActors)
	{
		if (ActorWeak.IsValid())
		{
			if (AActor* Actor = ActorWeak.Get())
			{
				if (Actor->FindComponentByClass<URemoteControlTrackerComponent>())
				{
					continue;
				}
#if WITH_EDITOR
				Actor->Modify();
#endif

				URemoteControlTrackerComponent* TrackerComponent = NewObject<URemoteControlTrackerComponent>(Actor
					, URemoteControlTrackerComponent::StaticClass()
					, MakeUniqueObjectName(Actor, URemoteControlTrackerComponent::StaticClass(), TEXT("RemoteControlTrackerComponent"))
					, RF_Transactional);

				Actor->AddInstanceComponent(TrackerComponent);
				TrackerComponent->RegisterComponent();
#if WITH_EDITOR
				Actor->RerunConstructionScripts();
#endif

				if (URemoteControlComponentsSubsystem* ControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
				{
#if WITH_EDITOR
					ControlComponentsSubsystem->Modify();
#endif
					ControlComponentsSubsystem->OnTrackedActorRegistered().Broadcast(Actor);
				}

				FRemoteControlComponentsUtils::RefreshTrackedProperties(Actor);
			}
		}
	}
#if WITH_EDITOR
	AddTrackerTransaction.Reset();
#endif
}

void FRemoteControlComponentsUtils::RemoveTrackerComponent(const TSet<TWeakObjectPtr<AActor>>& InActors)
{
#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("RemoveTrackerComponents", "Remove Tracker Component(s)"));
#endif

	for (const TWeakObjectPtr<AActor>& ActorWeak : InActors)
	{
		if (ActorWeak.IsValid())
		{
			if (AActor* Actor = ActorWeak.Get())
			{
				if (URemoteControlTrackerComponent* TrackerComponent = Actor->FindComponentByClass<URemoteControlTrackerComponent>())
				{
#if WITH_EDITOR
					Actor->Modify();
					TrackerComponent->Modify();

					if (URemoteControlPreset* Preset = FRemoteControlComponentsUtils::GetCurrentPreset(Actor))
					{
						Preset->Modify();
					}
#endif
					
					TrackerComponent->UnexposeAllProperties();
					Actor->RemoveInstanceComponent(TrackerComponent);
					TrackerComponent->DestroyComponent();

#if WITH_EDITOR
					Actor->RerunConstructionScripts();
#endif

					if (URemoteControlComponentsSubsystem* ControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
					{
#if WITH_EDITOR
						ControlComponentsSubsystem->Modify();
#endif
						ControlComponentsSubsystem->OnTrackedActorUnregistered().Broadcast(Actor);
					}
				}
			}
		}
	}
}

void FRemoteControlComponentsUtils::UnexposeAllProperties(AActor* InActor)
{
	if (!InActor)
	{
		return;
	}

	if (URemoteControlTrackerComponent* TrackerComponent = FRemoteControlComponentsUtils::GetTrackerComponent(InActor))
	{
#if WITH_EDITOR
		FScopedTransaction Transaction(LOCTEXT("UnexposeAllProperties", "Unexpose all properties"));
		InActor->Modify();

		if (URemoteControlPreset* Preset = FRemoteControlComponentsUtils::GetCurrentPreset(InActor))
		{
			Preset->Modify();
		}
#endif
		
		TrackerComponent->UnexposeAllProperties();
	}
}

TWeakPtr<FRemoteControlProperty> FRemoteControlComponentsUtils::ExposeProperty(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FRCFieldPathInfo& InPathInfo)
{
	if (!InPreset)
	{
		return nullptr;
	}

	FRemoteControlPresetExposeArgs Args;
	Args.Label = FString();
	Args.GroupId = FGuid();

	const FGuid& PropertyGuid = FRemoteControlComponentsUtils::GetPropertyGuid(InOwnerObject, InPathInfo);
	
	if (!InPreset->IsExposed(PropertyGuid))
	{
		return InPreset->ExposeProperty(InOwnerObject, InPathInfo, Args);
	}

	return nullptr;
}

void FRemoteControlComponentsUtils::UnexposeProperty(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FRCFieldPathInfo& InPathInfo)
{
	if (!InPreset)
	{
		return;
	}

	FRCFieldPathInfo FieldPathInfoLocal = InPathInfo;
	FieldPathInfoLocal.Resolve(InOwnerObject);
	
	if (const FProperty* Property = FieldPathInfoLocal.GetResolvedData().Field)
	{
		const FGuid PropertyGuid = FRemoteControlComponentsUtils::GetPresetPropertyGuid(InPreset, InOwnerObject, Property);

		if (InPreset->IsExposed(PropertyGuid))
		{
			InPreset->Unexpose(PropertyGuid);
		}
	}
}

TWeakPtr<FRemoteControlProperty> FRemoteControlComponentsUtils::GetExposedProperty(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FRCFieldPathInfo& InPathInfo)
{
	if (!InPreset)
	{
		return nullptr;
	}

	FRCFieldPathInfo FieldPathInfoLocal = InPathInfo;
	FieldPathInfoLocal.Resolve(InOwnerObject);
	
	if (const FProperty* Property = FieldPathInfoLocal.GetResolvedData().Field)
	{
		const FGuid PropertyGuid = FRemoteControlComponentsUtils::GetPresetPropertyGuid(InPreset, InOwnerObject, Property);

		return InPreset->GetExposedEntity<FRemoteControlProperty>(PropertyGuid);
	}

	return nullptr;
}

FName FRemoteControlComponentsUtils::GetExposedPropertyId(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FRCFieldPathInfo& InPathInfo)
{
	if (!InPreset)
	{
		return NAME_None;
	}

	const TWeakPtr<FRemoteControlProperty> RCPropertyWeak = FRemoteControlComponentsUtils::GetExposedProperty(InPreset, InOwnerObject, InPathInfo);		
	if (const TSharedPtr<FRemoteControlProperty> RCProperty = RCPropertyWeak.Pin())
	{
		return RCProperty->PropertyId;
	}

	return NAME_None;
}

void FRemoteControlComponentsUtils::SetExposedPropertyId(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FRCFieldPathInfo& InPathInfo, const FName& InPropertyId)
{
	if (!InPreset)
	{
		return;
	}

	const TWeakPtr<FRemoteControlProperty> RCPropertyWeak = FRemoteControlComponentsUtils::GetExposedProperty(InPreset, InOwnerObject, InPathInfo);		
	if (const TSharedPtr<FRemoteControlProperty> RCProperty = RCPropertyWeak.Pin())
	{
		RCProperty->PropertyId = InPropertyId;
		if (const TObjectPtr<URemoteControlPropertyIdRegistry>& PropertyRegistry = InPreset->GetPropertyIdRegistry())
		{
			PropertyRegistry->UpdateIdentifiedField(RCProperty.ToSharedRef());
		}		
	}
}

void FRemoteControlComponentsUtils::RefreshTrackedProperties(AActor* InActor)
{
	if (!InActor)
	{
		return;
	}
	
	constexpr bool bAddTrackerIfMissing = true;
	URemoteControlTrackerComponent* TrackerComponent = GetTrackerComponent(InActor, bAddTrackerIfMissing);
	
	if (!TrackerComponent)
	{
		return;
	}
	
	if (URemoteControlPreset* Preset = GetCurrentPreset(InActor))
	{
		const TArray<TWeakPtr<FRemoteControlProperty>>& RemoteControlExposedProperties = Preset->GetExposedEntities<FRemoteControlProperty>();
		for (const TWeakPtr<FRemoteControlProperty>& RCPropertyWeak : RemoteControlExposedProperties)
		{
			if (const TSharedPtr<FRemoteControlProperty>& RCProperty = RCPropertyWeak.Pin())
			{
				UObject* BoundObject = RCProperty->GetBoundObject();
				if (!BoundObject)
				{
					continue;
				}
				
				if (InActor == BoundObject->GetTypedOuter<AActor>())
				{
					const FRCFieldPathInfo& FieldPathInfo = RCProperty->FieldPathInfo;
					
					if (!TrackerComponent->IsTrackingProperty(FieldPathInfo, BoundObject))
					{
						TrackerComponent->AddTrackedProperty(FieldPathInfo, BoundObject);
					}				
				}
			}
		}
	}
}

FGuid FRemoteControlComponentsUtils::GetPropertyGuid(TConstArrayView<UObject*> InOuterObjects, const FProperty* InProperty)
{
	if (InOuterObjects.IsEmpty())
	{
		return FGuid();
	}
	
	if (UObject* OwnerObject = InOuterObjects[0])
	{
		if (URemoteControlPreset* Preset = FRemoteControlComponentsUtils::GetCurrentPreset(OwnerObject))
		{
			return FRemoteControlComponentsUtils::GetPresetPropertyGuid(Preset, OwnerObject, InProperty);
		}
	}

	return FGuid();
}

FGuid FRemoteControlComponentsUtils::GetPropertyGuid(UObject* InOwnerObject, const FRCFieldPathInfo& InFieldPathInfo)
{
	FRCFieldPathInfo FieldPathInfoLocal = InFieldPathInfo;
	FieldPathInfoLocal.Resolve(InOwnerObject);
	
	if (const FProperty* Property = FieldPathInfoLocal.GetResolvedData().Field)
	{
		return GetPropertyGuid({InOwnerObject}, Property);
	}

	return FGuid();
}

#undef LOCTEXT_NAMESPACE
