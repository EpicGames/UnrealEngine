// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlTrackerComponent.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Misc/TransactionObjectEvent.h"
#include "RemoteControlTrackerProperty.h"
#include "RemoteControlPreset.h"
#include "Subsystems/RemoteControlComponentsSubsystem.h"
#include "UObject/ObjectSaveContext.h"

URemoteControlPreset* URemoteControlTrackerComponent::GetCurrentPreset() const
{
	if (const URemoteControlComponentsSubsystem* RemoteControlPresetComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
	{
		if (const UWorld* World = GetTypedOuter<UWorld>())
		{
			if (URemoteControlPreset* Preset = RemoteControlPresetComponentsSubsystem->GetRegisteredPreset(World))
			{
				return Preset;
			}
		}
	}

	return nullptr;
}

bool URemoteControlTrackerComponent::HasTrackedProperties() const
{
	return !TrackedProperties.IsEmpty();
}

void URemoteControlTrackerComponent::AddTrackedProperty(const FRCFieldPathInfo& InFieldPathInfo, UObject* InOwnerObject)
{
	if (!IsTrackingProperty(InFieldPathInfo, InOwnerObject))
	{
		TrackedProperties.Add(FRemoteControlTrackerProperty(InFieldPathInfo, InOwnerObject));
	}
}

void URemoteControlTrackerComponent::RemoveTrackedProperty(const FRCFieldPathInfo& InFieldPathInfo, UObject* InOwnerObject)
{
	const int32 Index = GetTrackedPropertyIndex(InFieldPathInfo, InOwnerObject);
	if (Index != INDEX_NONE)
	{
		TrackedProperties.RemoveAt(Index);
	}
}

void URemoteControlTrackerComponent::ExposeAllProperties()
{
	if (URemoteControlPreset* Preset = GetCurrentPreset())
	{
		for (FRemoteControlTrackerProperty& TrackedProperty : TrackedProperties)
		{
			TrackedProperty.Expose(Preset);
		}
	}
}

void URemoteControlTrackerComponent::UnexposeAllProperties()
{
	TArray<FRemoteControlTrackerProperty> PropertiesToRemove = MoveTemp(TrackedProperties);

	for (FRemoteControlTrackerProperty& TrackedProperty : PropertiesToRemove)
	{
		TrackedProperty.Unexpose();
	}

	UnregisterPropertyIdChangeDelegate();
}

void URemoteControlTrackerComponent::RefreshAllPropertyIds()
{
	for (FRemoteControlTrackerProperty& TrackedProperty : TrackedProperties)
	{
		TrackedProperty.ReadPropertyIdFromPreset();
	}
}

void URemoteControlTrackerComponent::WriteAllPropertyIdsToPreset()
{
	UnregisterPropertyIdChangeDelegate();
	for (FRemoteControlTrackerProperty& TrackedProperty : TrackedProperties)
	{
		TrackedProperty.WritePropertyIdToPreset();
	}
	RegisterPropertyIdChangeDelegate();
}

bool URemoteControlTrackerComponent::IsTrackingProperty(const FRCFieldPathInfo& InFieldPathInfo, UObject* InOwnerObject) const
{
	const int32 Index = GetTrackedPropertyIndex(InFieldPathInfo, InOwnerObject);
	return Index != INDEX_NONE;
}

AActor* URemoteControlTrackerComponent::GetTrackedActor() const
{
	if (AActor* OuterActor = GetOwner())
	{
		return OuterActor;
	}

	return nullptr;
}

void URemoteControlTrackerComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	if (AActor* TrackedActor = GetTrackedActor())
	{
		if (URemoteControlComponentsSubsystem* RemoteControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
		{
			RemoteControlComponentsSubsystem->RegisterTrackedActor(TrackedActor);
		}
	}
}

void URemoteControlTrackerComponent::OnComponentDestroyed(bool bInDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bInDestroyingHierarchy);

	UnexposeAllProperties();

	if (AActor* TrackedActor = GetTrackedActor())
	{
		if (URemoteControlComponentsSubsystem* RemoteControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
		{
			RemoteControlComponentsSubsystem->UnregisterTrackedActor(TrackedActor);
		}
	}
}

void URemoteControlTrackerComponent::PostInitProperties()
{
	Super::PostInitProperties();

	RegisterTrackedActor();
	RegisterPropertyIdChangeDelegate();
}

void URemoteControlTrackerComponent::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);

	if (!bInDuplicateForPIE)
	{
		OnTrackerDuplicated();
	}
}

void URemoteControlTrackerComponent::PostEditImport()
{
	Super::PostEditImport();

	OnTrackerDuplicated();
}

void URemoteControlTrackerComponent::PostLoad()
{
	Super::PostLoad();
	RegisterPropertyIdChangeDelegate();
}

void URemoteControlTrackerComponent::PreSave(FObjectPreSaveContext InSaveContext)
{
	Super::PreSave(InSaveContext);

	CleanupProperties();
}

#if WITH_EDITOR
void URemoteControlTrackerComponent::PostTransacted(const FTransactionObjectEvent& InTransactionEvent)
{
	Super::PostTransacted(InTransactionEvent);

	if (InTransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		CleanupProperties();
		RefreshTracker();

		// In case the component was destroyed and created again with an Undo, let's try fixing potentially broken bindings
		if (URemoteControlPreset* Preset = GetCurrentPreset())
		{
			Preset->RebindUnboundEntities();
		}
	}
}
#endif

void URemoteControlTrackerComponent::RegisterTrackedActor() const
{
	if (AActor* OuterActor = GetTrackedActor())
	{
		if (URemoteControlComponentsSubsystem* RemoteControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
		{
			RemoteControlComponentsSubsystem->RegisterTrackedActor(OuterActor);
		}
	}
}

void URemoteControlTrackerComponent::UnregisterTrackedActor() const
{
	if (AActor* OuterActor = GetTrackedActor())
	{
		if (URemoteControlComponentsSubsystem* RemoteControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
		{
			RemoteControlComponentsSubsystem->UnregisterTrackedActor(OuterActor);
		}
	}
}

void URemoteControlTrackerComponent::OnTrackerDuplicated()
{
	CleanupProperties();

	// Early out if Preset duplication is being blocked from the conventional Duplicate->Renew->RefreshTracker
	// Not allowing preset guid renewal is analogous to duplicating the preset "as-is", and the tracker should not manipulate the preset
	if (!FRCPresetGuidRenewGuard::IsAllowingPresetGuidRenewal())
	{
		return;
	}

	RefreshTracker();
	WriteAllPropertyIdsToPreset();
	RegisterPropertyIdChangeDelegate();
}

void URemoteControlTrackerComponent::RefreshTracker()
{
	RegisterTrackedActor();
	RefreshExposedProperties();
}

void URemoteControlTrackerComponent::RefreshExposedProperties()
{
	MarkPropertiesForRefresh();
	ExposeAllProperties();
}

void URemoteControlTrackerComponent::MarkPropertiesForRefresh()
{
	for (FRemoteControlTrackerProperty& TrackedProperty : TrackedProperties)
	{
		TrackedProperty.MarkUnexposed();
	}
}

void URemoteControlTrackerComponent::CleanupProperties()
{
	for (TArray<FRemoteControlTrackerProperty>::TIterator PropertyIt = TrackedProperties.CreateIterator(); PropertyIt; ++PropertyIt)
	{
		FRemoteControlTrackerProperty TrackedProperty(*PropertyIt);

		if (!TrackedProperty.IsValid())
		{
			PropertyIt.RemoveCurrent();
		}
	}
}

void URemoteControlTrackerComponent::RegisterPropertyIdChangeDelegate()
{
	if (const URemoteControlPreset* Preset = GetCurrentPreset())
	{
		if (const TObjectPtr<URemoteControlPropertyIdRegistry>& PropertyRegistry = Preset->GetPropertyIdRegistry())
		{
			PropertyRegistry->OnPropertyIdUpdated().RemoveAll(this);
			PropertyRegistry->OnPropertyIdUpdated().AddUObject(this, &URemoteControlTrackerComponent::OnPropertyIdUpdated);
		}
	}
}

void URemoteControlTrackerComponent::UnregisterPropertyIdChangeDelegate() const
{
	if (const URemoteControlPreset* Preset = GetCurrentPreset())
	{
		if (const TObjectPtr<URemoteControlPropertyIdRegistry>& PropertyRegistry = Preset->GetPropertyIdRegistry())
		{
			PropertyRegistry->OnPropertyIdUpdated().RemoveAll(this);
		}
	}
}

void URemoteControlTrackerComponent::OnPropertyIdUpdated()
{
	RefreshAllPropertyIds();
}

int32 URemoteControlTrackerComponent::GetTrackedPropertyIndex(const FRCFieldPathInfo& InFieldPathInfo, UObject* InOwnerObject) const
{
	return TrackedProperties.IndexOfByPredicate([InFieldPathInfo, InOwnerObject](const FRemoteControlTrackerProperty& TrackerProperty)
	{
		return TrackerProperty.MatchesParameters(InFieldPathInfo, InOwnerObject);
	});
}
