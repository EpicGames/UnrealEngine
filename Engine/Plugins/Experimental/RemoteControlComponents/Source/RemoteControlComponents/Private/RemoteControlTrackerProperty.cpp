// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlTrackerProperty.h"
#include "IRemoteControlModule.h"
#include "RemoteControlComponentsUtils.h"
#include "RemoteControlPreset.h"

FRemoteControlTrackerProperty::FRemoteControlTrackerProperty()
{
}

FRemoteControlTrackerProperty::FRemoteControlTrackerProperty(const FRCFieldPathInfo& InFieldPathInfo, const TObjectPtr<UObject>& InOwnerObject, bool bInIsExposed)
	: FieldPathInfo(InFieldPathInfo)
	, OwnerObject(InOwnerObject)
	, bIsExposed(bInIsExposed)
{
	Resolve();
	ReadPropertyIdFromPreset();
}

bool FRemoteControlTrackerProperty::MatchesParameters(const FRCFieldPathInfo& InFieldPathInfo, const TObjectPtr<UObject>& InOwnerObject) const
{
	return FieldPathInfo == InFieldPathInfo
		&& OwnerObject == InOwnerObject;
}

URemoteControlPreset* FRemoteControlTrackerProperty::GetPreset() const
{
	return FRemoteControlComponentsUtils::GetCurrentPreset(OwnerObject.Get());
}

bool FRemoteControlTrackerProperty::IsExposedTo(const URemoteControlPreset* InPreset) const
{
	if (const URemoteControlPreset* CurrentPreset = GetPreset())
	{
		if (CurrentPreset == InPreset)
		{
			return IsExposed();
		}
	}

	return false;
}

void FRemoteControlTrackerProperty::Resolve() const
{
	if (OwnerObject.IsValid())
	{
		const_cast<FRemoteControlTrackerProperty*>(this)->FieldPathInfo.Resolve(OwnerObject.Get());
	}
}

void FRemoteControlTrackerProperty::MarkUnexposed()
{
	bIsExposed = false;
}

bool FRemoteControlTrackerProperty::IsValid() const
{
	return OwnerObject.IsValid();
}

bool FRemoteControlTrackerProperty::operator==(const FRemoteControlTrackerProperty& Other) const
{
	return FieldPathInfo == Other.FieldPathInfo && OwnerObject == Other.OwnerObject;
}

bool FRemoteControlTrackerProperty::operator!=(const FRemoteControlTrackerProperty& Other) const
{
	return !(*this == Other);
}

void FRemoteControlTrackerProperty::Expose(URemoteControlPreset* InRemoteControlPreset)
{
	if (IsExposedTo(InRemoteControlPreset))
	{
		// Property is already exposed to specified preset, just return
		return;
	}

	if (!InRemoteControlPreset || !IsValid())
	{
		return;
	}

	FRemoteControlComponentsUtils::ExposeProperty(InRemoteControlPreset, OwnerObject.Get(), FieldPathInfo);
	CurrentPresetWeak = MakeWeakObjectPtr<URemoteControlPreset>(InRemoteControlPreset);
	bIsExposed = true;
}

void FRemoteControlTrackerProperty::Unexpose()
{
	if (!IsExposed())
	{
		return;
	}

	if (!IsValid())
	{
		return;
	}

	if (URemoteControlPreset* Preset = GetPreset())
	{
		FRemoteControlComponentsUtils::UnexposeProperty(Preset, OwnerObject.Get(), FieldPathInfo);
	}

	bIsExposed = false;
}

void FRemoteControlTrackerProperty::ReadPropertyIdFromPreset()
{
	if (!IsValid())
	{
		return;
	}

	if (URemoteControlPreset* Preset = GetPreset())
	{
		const FName& Id = FRemoteControlComponentsUtils::GetExposedPropertyId(Preset, OwnerObject.Get(), FieldPathInfo);
		if (PropertyId != Id)
		{
			PropertyId = Id;
		}
	}
}

void FRemoteControlTrackerProperty::WritePropertyIdToPreset() const
{
	if (!IsExposed() || !IsValid())
	{
		return;
	}

	if (URemoteControlPreset* Preset = GetPreset())
	{
		FRemoteControlComponentsUtils::SetExposedPropertyId(Preset, OwnerObject.Get(), FieldPathInfo, PropertyId);
	}
}

uint32 GetTypeHash(const FRemoteControlTrackerProperty& InBroadcastControlId)
{
	return HashCombineFast(InBroadcastControlId.FieldPathInfo.PathHash, GetTypeHash(InBroadcastControlId.OwnerObject));
}
