// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/RemoteControlComponentsSubsystem.h"
#include "Engine/Engine.h"
#include "IRemoteControlModule.h"
#include "RemoteControlComponentsContext.h"
#include "RemoteControlComponentsUtils.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "RemoteControlTrackerComponent.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "RemoteControlComponentsSubsystem"

URemoteControlComponentsSubsystem* URemoteControlComponentsSubsystem::Get()
{
	if (GEngine)
	{
		if (URemoteControlComponentsSubsystem* RemoteControlComponentsSubsystem = GEngine->GetEngineSubsystem<URemoteControlComponentsSubsystem>())
		{
			return RemoteControlComponentsSubsystem;
		}
	}
	return nullptr;
}

void URemoteControlComponentsSubsystem::Initialize(FSubsystemCollectionBase& InCollection)
{
	Super::Initialize(InCollection);
	
#if WITH_EDITOR
	FWorldDelegates::OnCurrentLevelChanged.AddUObject(this, &URemoteControlComponentsSubsystem::OnLevelChanged);
	FWorldDelegates::OnWorldCleanup.AddUObject(this, &URemoteControlComponentsSubsystem::OnWorldCleanup);
	
	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().AddUObject(this, &URemoteControlComponentsSubsystem::OnLevelActorDeleted);
	}
#endif
}

void URemoteControlComponentsSubsystem::Deinitialize()
{
	Super::Deinitialize();

#if WITH_EDITOR
	FWorldDelegates::OnCurrentLevelChanged.RemoveAll(this);
	FWorldDelegates::OnWorldCleanup.RemoveAll(this);
	
	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}
#endif
}

void URemoteControlComponentsSubsystem::OnLevelChanged(ULevel* InNewLevel, ULevel* InOldLevel, UWorld* InWorld)
{
	ClearData();
}

void URemoteControlComponentsSubsystem::OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources)
{
	ClearData();
}

void URemoteControlComponentsSubsystem::ClearData()
{
	TrackedActorsContextMap.Empty();
	ActorsPendingRegister.Empty();
}

void URemoteControlComponentsSubsystem::CacheActorsToBeTracked(const TSet<TWeakObjectPtr<AActor>>& InActors)
{
	if (InActors.IsEmpty())
	{
		return;
	}
	
	for (TWeakObjectPtr<AActor> Actor: InActors)
	{
		if (Actor.IsValid())
		{
			ActorsPendingRegister.AddUnique(Actor.Get());
		}
	}
}

void URemoteControlComponentsSubsystem::RegisterTrackedActor(AActor* InActor)
{
	if (!InActor || IsActorTracked(InActor))
	{
		return;
	}

	bool bRegistered = false;
	if (const UWorld* ActorWorld = InActor->GetWorld())
	{
		if (const TSharedPtr<FRemoteControlComponentsContext>& TrackingContext = GetTrackedActorContext(ActorWorld))
		{
			if (TrackingContext->IsValid())
			{
				bRegistered = TrackingContext->RegisterActor(InActor);
			}
		}
	}

	if (bRegistered)
	{
		OnTrackedActorRegisteredDelegate.Broadcast(InActor);
	}
	else
	{
		// It was not possible to register the actor at this time.
		// We cache it so we can try to register it again when a new Preset is registered
		CacheActorsToBeTracked({InActor});
	}
}

void URemoteControlComponentsSubsystem::UnregisterTrackedActor(AActor* InActor)
{
	if (InActor)
	{
		if (const UWorld* ActorWorld = InActor->GetWorld())
		{
			if (const TSharedPtr<FRemoteControlComponentsContext>& TrackingContext = GetTrackedActorContext(ActorWorld))
			{
				if (TrackingContext->UnregisterActor(InActor))
				{
					OnTrackedActorUnregisteredDelegate.Broadcast(InActor);
				}
			}
		}

		ActorsPendingRegister.Remove(InActor);
	}
}

void URemoteControlComponentsSubsystem::RegisterPendingActors(const UWorld* InWorld)
{
	TArray<TWeakObjectPtr<AActor>> RegisteredActors;
	for (const TWeakObjectPtr<AActor>& PendingActor : ActorsPendingRegister)
	{
		if (!PendingActor.IsValid())
		{
			continue;
		}
			
		if (PendingActor->GetWorld() == InWorld)
		{
			RegisterTrackedActor(PendingActor.Get());
			RegisteredActors.AddUnique(PendingActor);
		}
	}

	for (const TWeakObjectPtr<AActor>& PendingActor : RegisteredActors)
	{
		ActorsPendingRegister.Remove(PendingActor);
	}
}

void URemoteControlComponentsSubsystem::RegisterPreset(URemoteControlPreset* InRemoteControlPreset)
{
	if (!InRemoteControlPreset)
	{
		return;
	}
	
	if (UWorld* PresetWorld = InRemoteControlPreset->GetTypedOuter<UWorld>())
	{
		const FObjectKey WorldKey(PresetWorld);
		if (!TrackedActorsContextMap.Contains(WorldKey))
		{
			TrackedActorsContextMap.Add(WorldKey, MakeShared<FRemoteControlComponentsContext>(PresetWorld, InRemoteControlPreset));
		}

		// A new Preset has been registered, let's try re-registering pending actors
		RegisterPendingActors(PresetWorld);

		InRemoteControlPreset->OnEntityExposed().AddUObject(this, &URemoteControlComponentsSubsystem::OnEntityExposed);
		InRemoteControlPreset->OnEntityUnexposed().AddUObject(this, &URemoteControlComponentsSubsystem::OnEntityUnexposed);
	}
}

void URemoteControlComponentsSubsystem::UnregisterPreset(URemoteControlPreset* InRemoteControlPreset)
{
	if (!InRemoteControlPreset)
	{
		return;
	}
	
	if (const UWorld* PresetWorld = InRemoteControlPreset->GetTypedOuter<UWorld>())
	{
		const FObjectKey WorldKey(PresetWorld);
		if (TrackedActorsContextMap.Contains(WorldKey))
		{
			TrackedActorsContextMap.Remove(WorldKey);
		}
	}

	InRemoteControlPreset->OnEntityExposed().RemoveAll(this);
	InRemoteControlPreset->OnEntityUnexposed().RemoveAll(this);
}

URemoteControlPreset* URemoteControlComponentsSubsystem::GetRegisteredPreset(const UWorld* InWorld) const
{
	if (InWorld)
	{
		if (const TSharedPtr<FRemoteControlComponentsContext>& TrackingContext = GetTrackedActorContext(InWorld))
		{
			if (URemoteControlPreset* RegisteredPreset = TrackingContext->GetPreset())
			{
				return RegisteredPreset;
			}
		}
	}
	
	return nullptr;
}

URemoteControlPreset* URemoteControlComponentsSubsystem::GetRegisteredPreset(const UObject* InObject) const
{
	if (!InObject)
	{
		return nullptr;
	}
	
	if (const UWorld* World = InObject->GetTypedOuter<UWorld>())
	{
		return GetRegisteredPreset(World);
	}

	return nullptr;
}

bool URemoteControlComponentsSubsystem::IsActorTracked(const AActor* InActor) const
{
	if (!InActor)
	{
		return false;
	}

	if (const UWorld* ActorWorld = InActor->GetWorld())
	{
		if (const TSharedPtr<FRemoteControlComponentsContext>& TrackingContext = GetTrackedActorContext(ActorWorld))
		{
			return TrackingContext->IsActorRegistered(InActor);
		}
	}

	return false;
}

bool URemoteControlComponentsSubsystem::IsPresetRegistered(const URemoteControlPreset* InPreset) const
{
	if (!InPreset)
	{
		return false;
	}

	if (const UWorld* PresetWorld = InPreset->GetTypedOuter<UWorld>())
	{	
		if (const URemoteControlPreset* RegisteredPreset = GetRegisteredPreset(PresetWorld))
		{
			return RegisteredPreset == InPreset;
		}
	}

	return false;
}

void URemoteControlComponentsSubsystem::OnEntityExposed(URemoteControlPreset* InRemoteControlPreset, const FGuid& InGuid)
{
	if (InRemoteControlPreset && InGuid.IsValid())
	{
		if (const TSharedPtr<FRemoteControlProperty>& RCProperty = InRemoteControlPreset->GetExposedEntity<FRemoteControlProperty>(InGuid).Pin())
		{
			if (RCProperty.IsValid())
			{
				constexpr bool bCleanDuplicates = true;
				const FRCFieldPathInfo PathInfo(RCProperty->FieldPathInfo.ToString(), bCleanDuplicates);

				if (UObject* OwnerObject = RCProperty->GetBoundObject())
				{
					if (AActor* OwnerActor = OwnerObject->GetTypedOuter<AActor>())
					{
						constexpr bool bAddTrackerIfMissing = true;
						if (URemoteControlTrackerComponent* TrackerComponent = FRemoteControlComponentsUtils::GetTrackerComponent(OwnerActor, bAddTrackerIfMissing))
						{
							TrackerComponent->AddTrackedProperty(PathInfo, OwnerObject);
						}
					}
				}
			}
		}
	}
}

void URemoteControlComponentsSubsystem::OnEntityUnexposed(URemoteControlPreset* InRemoteControlPreset, const FGuid& InGuid)
{
	if (!InRemoteControlPreset || !InGuid.IsValid())
	{
		return;
	}

	const TSharedPtr<FRemoteControlProperty>& RCProperty = InRemoteControlPreset->GetExposedEntity<FRemoteControlProperty>(InGuid).Pin();
	if (!RCProperty.IsValid())
	{
		return;
	}

	UObject* OwnerObject = RCProperty->GetBoundObject();
	if (!OwnerObject)
	{
		return;
	}

	if (AActor* OwnerActor = OwnerObject->GetTypedOuter<AActor>())
	{
		// Look for a tracker component, and remove the property from the tracked properties set
		if (URemoteControlTrackerComponent* TrackerComponent = FRemoteControlComponentsUtils::GetTrackerComponent(OwnerActor))
		{			
			if (IsRegisteredTrackedActor(OwnerActor))
			{
				constexpr bool bCleanDuplicates = true;
				const FRCFieldPathInfo PathInfo(RCProperty->FieldPathInfo.ToString(), bCleanDuplicates);
				TrackerComponent->RemoveTrackedProperty(PathInfo, OwnerObject);
			}
		}
	}
}

void URemoteControlComponentsSubsystem::OnLevelActorDeleted(AActor* InActor)
{
	if (!InActor)
	{
		return;
	}
	
	UnregisterTrackedActor(InActor);
	FRemoteControlComponentsUtils::UnexposeAllProperties(InActor);
}

bool URemoteControlComponentsSubsystem::IsRegisteredActor(const AActor* InActor) const
{
	return IsRegisteredTrackedActor(InActor);
}

bool URemoteControlComponentsSubsystem::IsRegisteredTrackedActor(const AActor* InActor) const
{
	if (const UWorld* ActorWorld = InActor->GetWorld())
	{
		if (const TSharedPtr<FRemoteControlComponentsContext>& TrackingContext = GetTrackedActorContext(ActorWorld))
		{
			return TrackingContext->IsActorRegistered(InActor);
		}
	}
	
	return false;
}

TSharedPtr<FRemoteControlComponentsContext> URemoteControlComponentsSubsystem::GetTrackedActorContext(const UWorld* InWorld) const
{
	if (InWorld)
	{
		const FObjectKey WorldKey(InWorld);
		if (const TSharedPtr<FRemoteControlComponentsContext>* TrackingContextPtr = TrackedActorsContextMap.Find(WorldKey))
		{
			return *TrackingContextPtr;
		}
	}

	return {};
}

#undef LOCTEXT_NAMESPACE
