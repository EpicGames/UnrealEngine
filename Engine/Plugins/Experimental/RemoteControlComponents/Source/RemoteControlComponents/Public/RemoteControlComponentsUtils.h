// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class URemoteControlTrackerComponent;
class URemoteControlPreset;

struct FRCFieldPathInfo;
struct FRemoteControlProperty;

class FRemoteControlComponentsUtils
{
	friend class URemoteControlComponentsSubsystem;
	friend struct FRemoteControlTrackerProperty;

public:
	/** Add a RemoteControlTrackerComponent to the specified Actors */
	REMOTECONTROLCOMPONENTS_API static void AddTrackerComponent(const TSet<TWeakObjectPtr<AActor>>& InActors, bool bInShouldTransact = true);

	/** Removes the RemoteControlTrackerComponent from the specified Actors */
	REMOTECONTROLCOMPONENTS_API static void RemoveTrackerComponent(const TSet<TWeakObjectPtr<AActor>>& InActors);

	/**
	 * Unexpose all Tracked Properties of the specified Actor.
	 * It will also stop tracking all unexposed properties.
	 */
	REMOTECONTROLCOMPONENTS_API static void UnexposeAllProperties(AActor* InActor);

	/**
	 * Gets the tracker component for the specified Actor. Tracker can be optionally added if missing.
	 * @param InActor: the Actor for which we want to retrieve the Tracker Component. Object 
	 * @param bInAddTrackerIfMissing: if true, function will add a tracker component when one is missing
	 * @return The found or created tracker.
	 */
	static URemoteControlTrackerComponent* GetTrackerComponent(AActor* InActor, bool bInAddTrackerIfMissing = false);

	/** Gets the Remote Control Preset currently handling the specified Object */
	static URemoteControlPreset* GetCurrentPreset(const UObject* InObject);

	/** Gets the Remote Control Preset currently handling the specified World */
	static URemoteControlPreset* GetCurrentPreset(const UWorld* InWorld);

	/** Call this function on any Actor to ensure its exposed RC properties are actually tracked */
	static void RefreshTrackedProperties(AActor* InActor);
	
private:
	/**
	 * Expose a Property to the specified Preset
	 * @param InPreset the Preset where the property should be exposed
	 * @param InOwnerObject the object owning the property
	 * @param InPathInfo Field Path Info for the desire property
	 * @return The exposed property
	 */
	static TWeakPtr<FRemoteControlProperty> ExposeProperty(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FRCFieldPathInfo& InPathInfo);

	/**
	 * Unexpose a Property in the specified Preset
	 * @param InPreset the Preset where the property is currently exposed
	 * @param InOwnerObject the object owning the property
	 * @param InPathInfo Field Path Info for the desire property
	 */
	static void UnexposeProperty(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FRCFieldPathInfo& InPathInfo);

	static TWeakPtr<FRemoteControlProperty> GetExposedProperty(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FRCFieldPathInfo& InPathInfo);

	/**
	 * Returns the field identifier for the exposed property described by the provided arguments.
	 * @param InPreset the Preset where the property is currently exposed
	 * @param InOwnerObject the object owning the property
	 * @param InPathInfo Field Path Info for the desire property
	 */
	static FName GetExposedPropertyId(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FRCFieldPathInfo& InPathInfo);

	/**
	 * Sets the field identifier for the exposed property described by the provided arguments.
	 * @param InPreset the Preset where the property is currently exposed
	 * @param InOwnerObject the object owning the property
	 * @param InPathInfo Field Path Info for the desire property
	 * @param InPropertyId The new Property Id value to be set
	 */
	static void SetExposedPropertyId(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FRCFieldPathInfo& InPathInfo, const FName& InPropertyId);
	
	static FGuid GetPropertyGuid(TConstArrayView<UObject*> InOuterObjects, const FProperty* InProperty);
	static FGuid GetPropertyGuid(UObject* InOwnerObject, const FRCFieldPathInfo& InFieldPathInfo);

	/**
	 * Returns the ID of a property, if exposed in a Preset for the specified Owner Object
	 * @param InPreset The Preset containing the exposed Property
	 * @param InOwnerObject The Object owning the bound property
	 * @param InProperty The property to look for
	 * @return the ID of the property
	 */
	static FGuid GetPresetPropertyGuid(URemoteControlPreset* InPreset, UObject* InOwnerObject, const FProperty* InProperty);
};
