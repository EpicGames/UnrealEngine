// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "RemoteControlTrackerProperty.h"
#include "RemoteControlTrackerComponent.generated.h"

struct FRCFieldPathInfo;
class URemoteControlPreset;

/**
 * A component keeping track of properties currently exposed to Remote Control.
 * Supports auto-exposing of properties upon duplication.
 * e.g. duplicating an Actor with this component will result in the same properties exposed by Source Actor
 * to be automatically exposed by Duplicate Actor.
 */
UCLASS(MinimalAPI)
class URemoteControlTrackerComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	/** Gets the Current Preset this Tracker Component points to */
	REMOTECONTROLCOMPONENTS_API URemoteControlPreset* GetCurrentPreset() const;

	/** returns true if Tracker Component has at least one tracked/exposed property */
	REMOTECONTROLCOMPONENTS_API bool HasTrackedProperties() const;

	/** Add the specified property to the list of properties handled by this Tracker Component */
	void AddTrackedProperty(const FRCFieldPathInfo& InFieldPathInfo, UObject* InOwnerObject);

	/** Remove the specified property from the list of properties handled by this Tracker Component */
	void RemoveTrackedProperty(const FRCFieldPathInfo& InFieldPathInfo, UObject* InOwnerObject);

	/** Expose all Tracked Properties handled by this component to the current Remote Control Preset */
	void ExposeAllProperties();

	/**
	 * Unexpose all Tracked Properties handled by this component from the current Remote Control Preset.
	 * It will also stop tracking all unexposed properties.
	 */
	void UnexposeAllProperties();

	/**
	 * Retrieves the Property Id for all the exposed tracked properties from the current Preset
	 * (Tracker --> Preset)
	 */
	void RefreshAllPropertyIds();

	/**
	 * Writes the Property Id saved in the Tracker Properties onto the exposed properties of the current Preset
	 * (Preset --> Tracker)
	 */
	void WriteAllPropertyIdsToPreset();

	/** returns true if Tracker Component is tracking the property for the specified FieldPathInfo and Owner Object */
	bool IsTrackingProperty(const FRCFieldPathInfo& InFieldPathInfo, UObject* InOwnerObject) const;

	/** Get the Actor owning this Tracker Component */
	AActor* GetTrackedActor() const;

protected:
	//~ Begin UActorComponent Interface
	virtual void OnComponentCreated() override;
	virtual void OnComponentDestroyed(bool bInDestroyingHierarchy) override;
	//~ End UActorComponent

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	virtual void PostDuplicate(bool bInDuplicateForPIE) override;
	virtual void PostEditImport() override;
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext InSaveContext) override;
#if WITH_EDITOR
	virtual void PostTransacted(const FTransactionObjectEvent& InTransactionEvent) override;
#endif
	//~ End UObject

private:
	void RegisterTrackedActor() const;
	void UnregisterTrackedActor() const;

	void OnTrackerDuplicated();

	void RefreshTracker();
	void RefreshExposedProperties();
	void MarkPropertiesForRefresh();
	void CleanupProperties();

	void RegisterPropertyIdChangeDelegate();
	void UnregisterPropertyIdChangeDelegate() const;
	void OnPropertyIdUpdated();

	int32 GetTrackedPropertyIndex(const FRCFieldPathInfo& InFieldPathInfo, UObject* InOwnerObject) const;

	UPROPERTY()
	TArray<FRemoteControlTrackerProperty> TrackedProperties;
};
