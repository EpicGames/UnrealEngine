// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteControlFieldPath.h"
#include "RemoteControlTrackerProperty.generated.h"

class URemoteControlPreset;

/**
 * Represents an exposed Remote Control Property tracked by a Remote Control Tracker Component
 * Its Field Path Info and Owner are used to represent the RC Property.
 */
USTRUCT()
struct FRemoteControlTrackerProperty
{
	GENERATED_BODY()

public:
	FRemoteControlTrackerProperty();

	/**
	 * Creates and initializes a Tracked Property
	 * @param InFieldPathInfo: Field Path Info to reach the property
	 * @param InOwnerObject: the object owning the Tracked Property
	 * @param bInIsExposed: is the property already exposed? Default is true.
	 */
	FRemoteControlTrackerProperty(const FRCFieldPathInfo& InFieldPathInfo, const TObjectPtr<UObject>& InOwnerObject, bool bInIsExposed = true);

	/** Checks if this Tracked Property is matching the one described by the specified parameters */
	bool MatchesParameters(const FRCFieldPathInfo& InFieldPathInfo, const TObjectPtr<UObject>& InOwnerObject) const;
	
	/** Returns the Remote Control Preset where this property is or will be exposed */
	URemoteControlPreset* GetPreset() const;

	/** Is the tracked property exposed to the specified Preset? */
	bool IsExposedTo(const URemoteControlPreset* InPreset) const;
	
	/** Is this Tracked Property exposed? */
	bool IsExposed() const { return bIsExposed; }

	/** Expose this Tracked Property to the specified Preset */
	void Expose(URemoteControlPreset* InRemoteControlPreset);

	/** Unexpose this Tracked Property from its Preset */
	void Unexpose();

	/** Retrieves the current Property Id for the currently tracked property */
	void ReadPropertyIdFromPreset();

	/** Writes the Property Id saved in this Tracker Property to the tracked property in the Preset */
	void WritePropertyIdToPreset() const;

	/**
	 * Marks this Tracked Property as not exposed, even if it might be.
	 * Useful to force refresh the state of a Tracked Property.
	 */
	void MarkUnexposed();

	/** Check if this property can still be exposed (internally verifies if Owner Object is valid) */
	bool IsValid() const;

	const FRCFieldPathInfo& GetFieldPathInfo() const { return FieldPathInfo; }

	bool operator==(const FRemoteControlTrackerProperty& Other) const;

	bool operator!=(const FRemoteControlTrackerProperty& Other) const;

	friend uint32 GetTypeHash(const FRemoteControlTrackerProperty& InBroadcastControlId);

private:
	/** Resolves the Tracked Property FieldPathInfo with its Owner */
	void Resolve() const;

	/** Field Path Info, matching the same one found in the Remote Control Preset */
	UPROPERTY()
	FRCFieldPathInfo FieldPathInfo;

	/** The Property Id of the exposed property tracked by this Tracker Property */
	UPROPERTY()
	FName PropertyId = NAME_None;

	/** Owner of the property handled by this Tracked Component */
	UPROPERTY()
	TWeakObjectPtr<UObject> OwnerObject = nullptr;

	/** Is this Tracked Property exposed? */
	UPROPERTY()
	bool bIsExposed = false;

	/** Caching the current Remote Control Preset */
	UPROPERTY()
	TWeakObjectPtr<URemoteControlPreset> CurrentPresetWeak = nullptr;
};
