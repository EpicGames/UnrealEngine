// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"
#include "RemoteControlComponentsSubsystem.generated.h"

class FSubsystemCollectionBase;
class UObject;
class URemoteControlPreset;
class URemoteControlTrackerComponent;
struct FRCFieldPathInfo;
struct FRemoteControlComponentsContext;

UCLASS(MinimalAPI)
class URemoteControlComponentsSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	/** Get this subsystem instance */
	REMOTECONTROLCOMPONENTS_API static URemoteControlComponentsSubsystem* Get();

	/** Register the specified Preset, along with the World it resides in. */
	REMOTECONTROLCOMPONENTS_API void RegisterPreset(URemoteControlPreset* InRemoteControlPreset);

	/** Unregister the specified Preset */
	REMOTECONTROLCOMPONENTS_API void UnregisterPreset(URemoteControlPreset* InRemoteControlPreset);

	/** Marks the specified Actor as a Tracked Actor, so that the Subsystem knows about it */
	void RegisterTrackedActor(AActor* InActor);
	
	/** Removes the specified Actor from the tracked actors register */
	void UnregisterTrackedActor(AActor* InActor);
	
	/** Checks if the specified Actor is currently being tracked by the subsystem */
	bool IsActorTracked(const AActor* InActor) const;

	/** Checks if the specified Preset is currently registered */
	bool IsPresetRegistered(const URemoteControlPreset* InPreset) const;

	/** Returns the Preset currently associated with the specified World */
	URemoteControlPreset* GetRegisteredPreset(const UWorld* InWorld) const;

	/** Returns the Preset currently associated with the specified Object */
	URemoteControlPreset* GetRegisteredPreset(const UObject* InObject) const;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnActorRegisterStateUpdate, AActor* /* InActor */)
	/** Fires whenever an actor is registered for Tracking */
	FOnActorRegisterStateUpdate& OnTrackedActorRegistered() { return OnTrackedActorRegisteredDelegate; }

	/** Fires whenever an actor is unregistered from Tracking */
	FOnActorRegisterStateUpdate& OnTrackedActorUnregistered() { return OnTrackedActorUnregisteredDelegate; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnActivePresetChanged, URemoteControlPreset* /* InActor */)
	/** Fires when the active Preset is updated */
	FOnActivePresetChanged& OnActivePresetChanged() { return OnActivePresetChangedDelegate; }

private:
	FOnActorRegisterStateUpdate OnTrackedActorRegisteredDelegate;	
	FOnActorRegisterStateUpdate OnTrackedActorUnregisteredDelegate;
	FOnActivePresetChanged OnActivePresetChangedDelegate;

	//~ Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& InCollection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem

	void OnLevelChanged(ULevel* InNewLevel, ULevel* InOldLevel, UWorld* InWorld);
	void OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources);
	void OnLevelActorDeleted(AActor* InActor);

	void ClearData();
	
	void OnEntityExposed(URemoteControlPreset* InRemoteControlPreset, const FGuid& InGuid);
	void OnEntityUnexposed(URemoteControlPreset* InRemoteControlPreset, const FGuid& InGuid);

	void CacheActorsToBeTracked(const TSet<TWeakObjectPtr<AActor>>& InActors);
	void RegisterPendingActors(const UWorld* InWorld);

	bool IsRegisteredActor(const AActor* InActor) const;
	bool IsRegisteredTrackedActor(const AActor* InActor) const;

	TSharedPtr<FRemoteControlComponentsContext> GetTrackedActorContext(const UWorld* InWorld) const;

	/** Utility container used to store actors for later tracking in case registering doesn't work right away */
	TArray<TWeakObjectPtr<AActor>> ActorsPendingRegister;

	TMap<FObjectKey, TSharedPtr<FRemoteControlComponentsContext>> TrackedActorsContextMap;
};
