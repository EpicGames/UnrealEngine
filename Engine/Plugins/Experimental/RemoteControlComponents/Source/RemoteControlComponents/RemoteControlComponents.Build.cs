// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RemoteControlComponents : ModuleRules
{
	public RemoteControlComponents(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"RemoteControl"
			}
			);
	

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
			}
			);


		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"RemoteControlUI",
				"UnrealEd",
			});
		}
	}
}
