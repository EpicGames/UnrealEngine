// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Styles/RemoteControlComponentsEditorStyle.h"

class FRemoteControlComponentsEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override
	{
		// Initialize the Style on Startup Module
		FRemoteControlComponentsEditorStyle::Get();
	}
	//~ End IModuleInterface
};

IMPLEMENT_MODULE(FRemoteControlComponentsEditorModule, RemoteControlComponentsEditor)
