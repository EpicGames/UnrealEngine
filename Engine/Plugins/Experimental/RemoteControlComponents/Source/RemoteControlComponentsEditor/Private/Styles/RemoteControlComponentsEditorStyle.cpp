// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlComponentsEditorStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FRemoteControlComponentsEditorStyle::FRemoteControlComponentsEditorStyle()
	: FSlateStyleSet(TEXT("RemoteControlComponentsEditor"))
{
	const FVector2f Icon16x16(16.f, 16.f);
	
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	ContentRootDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources"));

	Set("ClassIcon.RemoteControlTrackerComponent", new IMAGE_BRUSH_SVG("EditorIcons/RemoteControlTracker", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FRemoteControlComponentsEditorStyle::~FRemoteControlComponentsEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
