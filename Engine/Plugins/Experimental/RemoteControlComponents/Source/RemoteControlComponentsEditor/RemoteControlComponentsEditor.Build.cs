// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RemoteControlComponentsEditor : ModuleRules
{
    public RemoteControlComponentsEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "Engine",
                "EditorSubsystem",
                "Projects",
                "RemoteControl",
				"RemoteControlComponents",
                "Slate",
                "SlateCore",
                "UnrealEd"
            }
        );
    }
}